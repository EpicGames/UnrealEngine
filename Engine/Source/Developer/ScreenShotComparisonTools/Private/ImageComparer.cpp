// Copyright Epic Games, Inc. All Rights Reserved.

#include "ImageComparer.h"

#include "Async/ParallelFor.h"
#include "IImageWrapper.h"
#include "IImageWrapperModule.h"
#include "ImageWrapperHelper.h"
#include "HAL/LowLevelMemTracker.h"
#include "Misc/FileHelper.h"
#include "Misc/Paths.h"
#include "Modules/ModuleManager.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ImageComparer)


#define LOCTEXT_NAMESPACE "ImageComparer"

const FImageTolerance FImageTolerance::DefaultIgnoreNothing(0, 0, 0, 0, 0, 255, false, false, 0.00f, 0.00f);
const FImageTolerance FImageTolerance::DefaultIgnoreLess(16, 16, 16, 16, 16, 240, false, false, 0.02f, 0.02f);
const FImageTolerance FImageTolerance::DefaultIgnoreAntiAliasing(32, 32, 32, 32, 64, 96, true, false, 0.02f, 0.02f);
const FImageTolerance FImageTolerance::DefaultIgnoreColors(16, 16, 16, 16, 16, 240, false, true, 0.02f, 0.02f);


class FImageDelta
{
public:
	int32 Width;
	int32 Height;
	TArray<uint8> Image;

	FImageDelta(int32 InWidth, int32 InHeight)
		: Width(InWidth)
		, Height(InHeight)
	{
		Image.SetNumUninitialized(Width * Height * 4);
	}

	FString OutputComparisonFile;

	FORCEINLINE void SetPixel(int32 X, int32 Y, FColor Color)
	{
		int32 Offset = ( Y * Width + X ) * 4;
		check(Offset < ( Width * Height * 4 ));

		Image[Offset] = Color.R;
		Image[Offset + 1] = Color.G;
		Image[Offset + 2] = Color.B;
		Image[Offset + 3] = Color.A;
	}

	FORCEINLINE void SetPixelGrayScale(int32 X, int32 Y, FColor Color)
	{
		int32 Offset = ( Y * Width + X ) * 4;
		check(Offset < ( Width * Height * 4 ));

		const double Brightness = FPixelOperations::GetLuminance(Color);

		Image[Offset] = Brightness;
		Image[Offset + 1] = Brightness;
		Image[Offset + 2] = Brightness;
		Image[Offset + 3] = Color.A;
	}

	FORCEINLINE void SetClearPixel(int32 X, int32 Y)
	{
		int32 Offset = ( Y * Width + X ) * 4;
		check(Offset < ( Width * Height * 4 ));

		Image[Offset] = 0;
		Image[Offset + 1] = 0;
		Image[Offset + 2] = 0;
		Image[Offset + 3] = 255;
	}

	FORCEINLINE void SetErrorPixel(int32 X, int32 Y, FColor ErrorColor = FColor(255, 255, 255, 255))
	{
		int32 Offset = ( Y * Width + X ) * 4;
		check(Offset < ( Width * Height * 4 ));

		Image[Offset] = ErrorColor.R;
		Image[Offset + 1] = ErrorColor.G;
		Image[Offset + 2] = ErrorColor.B;
		Image[Offset + 3] = ErrorColor.A;
	}

	void Save(FString OutputDeltaFile)
	{
		// if the user supplies no path we use the temp dir
		FString TempDeltaFile = OutputDeltaFile.IsEmpty() ? FString(FPlatformProcess::UserTempDir()) : OutputDeltaFile;

		// if this is just a path, create a temp filename
		if (FPaths::GetExtension(TempDeltaFile).IsEmpty())
		{
			TempDeltaFile = FPaths::CreateTempFilename(*TempDeltaFile, TEXT("ImageCompare-"), TEXT(".png"));
		}

		IImageWrapperModule& ImageWrapperModule = FModuleManager::GetModuleChecked<IImageWrapperModule>(FName("ImageWrapper"));
		TSharedPtr<IImageWrapper> ImageWriter = ImageWrapperModule.CreateImageWrapper(EImageFormat::PNG);

		if ( ImageWriter.IsValid() )
		{
			if ( ImageWriter->SetRaw(Image.GetData(), Image.Num(), Width, Height, ERGBFormat::RGBA, 8) )
			{
				TArray64<uint8> PngData = ImageWriter->GetCompressed();

				if ( FFileHelper::SaveArrayToFile(PngData, *TempDeltaFile) )
				{
					OutputComparisonFile = TempDeltaFile;
				}
			}
		}
	}
};

float FPixelOperations::GetHue(const FColor& Color)
{
	float R = Color.R / 255.0f;
	float G = Color.G / 255.0f;
	float B = Color.B / 255.0f;

	float Max = FMath::Max3(R, G, B);
	float Min = FMath::Min3(R, G, B);

	if ( Max == Min )
	{
		return 0; // achromatic
	}
	else
	{
		float Hue = 0;
		float Delta = Max - Min;

		if ( Max == R )
		{
			Hue = ( G - B ) / Delta + ( G < B ? 6 : 0 );
		}
		else if ( Max == G )
		{
			Hue = ( B - R ) / Delta + 2;
		}
		else if ( Max == B )
		{
			Hue = ( R - G ) / Delta + 4;
		}

		return Hue /= 6.0f;
	}
}

bool FPixelOperations::IsAntialiased(const FColor& SourcePixel, const FComparableImage* Image, int32 X, int32 Y, const FImageTolerance& Tolerance)
{
	int32 hasHighContrastSibling = 0;
	int32 hasSiblingWithDifferentHue = 0;
	int32 hasEquivalentSibling = 0;

	float SourceHue = GetHue(SourcePixel);

	int32 Distance = 1;
	for ( int32 i = Distance * -1; i <= Distance; i++ )
	{
		for ( int32 j = Distance * -1; j <= Distance; j++ )
		{
			if ( i == 0 && j == 0 )
			{
				// ignore source pixel
			}
			else
			{
				if ( !Image->CanGetPixel(X + j, Y + i) )
				{
					continue;
				}

				FColor TargetPixel = Image->GetPixel(X + j, Y + i);

				double TargetPixelBrightness = GetLuminance(TargetPixel);
				float TargetPixelHue = GetHue(TargetPixel);

				if ( FPixelOperations::IsContrasting(SourcePixel, TargetPixel, Tolerance) )
				{
					hasHighContrastSibling++;
				}

				if ( FPixelOperations::IsRGBSame(SourcePixel, TargetPixel) )
				{
					hasEquivalentSibling++;
				}

				if ( FMath::Abs(SourceHue - TargetPixelHue) > 0.3 )
				{
					hasSiblingWithDifferentHue++;
				}

				if ( hasSiblingWithDifferentHue > 1 || hasHighContrastSibling > 1 )
				{
					return true;
				}
			}
		}
	}

	if ( hasEquivalentSibling < 2 )
	{
		return true;
	}

	return false;
}

FComparisonReport::FComparisonReport(const FString& InReportRootDirectory, const FString& InReportFile)
{
	ReportRootDirectory = InReportRootDirectory;
	ReportFile = InReportFile;
	ReportPath = FPaths::GetPath(InReportFile);
}

bool FComparableImage::LoadFile(const FString& ImagePath, FText& OutError)
{
	TArray64<uint8> ImageData;
	const bool OpenSuccess = FFileHelper::LoadFileToArray(ImageData, *ImagePath);

	if ( !OpenSuccess )
	{
		OutError = LOCTEXT("ErrorOpeningImageA", "Unable to read image");
		return false;
	}

	const FString ImageExtension = FPaths::GetExtension(ImagePath);
	return LoadCompressedData(ImageData.GetData(), ImageData.Num(), ImageExtension, OutError);
}

bool FComparableImage::LoadCompressedData(const void* CompressedData, int64 CompressedSize, const FString& ImageExtension, FText& OutError)
{
	LLM_SCOPE_BYNAME(TEXT("AutomationTest/ImageCompare"));
	const EImageFormat ImageFormat = ImageWrapperHelper::GetImageFormat(ImageExtension);

	IImageWrapperModule& ImageWrapperModule = FModuleManager::GetModuleChecked<IImageWrapperModule>(FName("ImageWrapper"));
	TSharedPtr<IImageWrapper> ImageReader = ImageWrapperModule.CreateImageWrapper(ImageFormat);

	if ( !ImageReader.IsValid() )
	{
		OutError = FText::Format(LOCTEXT("ImageWrapperMissing", "Unable to locate image processor for file format {0}"), FText::FromString(ImageExtension));
		return false;
	}

	if ( !ImageReader->SetCompressed(CompressedData, CompressedSize) )
	{
		OutError = LOCTEXT("ErrorParsingImageA", "Unable to parse image");
		return false;
	}

	if ( !ImageReader->GetRaw(ERGBFormat::RGBA, 8, Bytes) )
	{
		OutError = LOCTEXT("ErrorReadingRawDataA", "Unable to decompress image");
		return false;
	}

	Width = ImageReader->GetWidth();
	Height = ImageReader->GetHeight();
	return true;
}

FImageComparisonResult FImageComparer::Compare(const FString& ImagePathA, const FString& ImagePathB, FImageTolerance Tolerance, const FString& OutDeltaPath)
{
	FText ErrorA, ErrorB;
	FComparableImage ImageA, ImageB;

	if ( !ImageA.LoadFile(ImagePathA, ErrorA) )
	{
		FImageComparisonResult Results;
		Results.ApprovedFilePath = ImagePathA;
		Results.IncomingFilePath = ImagePathB;
		Results.ErrorMessage = ErrorA;
		return Results;
	}

	if ( !ImageB.LoadFile(ImagePathB, ErrorB) )
	{
		FImageComparisonResult Results;
		Results.ApprovedFilePath = ImagePathA;
		Results.IncomingFilePath = ImagePathB;
		Results.ErrorMessage = ErrorB;
		return Results;
	}

	FImageComparisonResult Results = Compare(&ImageA, &ImageB, Tolerance, OutDeltaPath);
	Results.ApprovedFilePath = ImagePathA;
	Results.IncomingFilePath = ImagePathB;
	return Results;
}

FImageComparisonResult FImageComparer::Compare(const FComparableImage* ImageA, const FComparableImage* ImageB, FImageTolerance Tolerance, const FString& OutDeltaPath)
{
	LLM_SCOPE_BYNAME(TEXT("AutomationTest/ImageCompare"));
	FImageComparisonResult Results;

	// Compare the smallest shared dimensions, this will be a forced failure
	// but still offer a delta for context to the result reviewer
	const int32 MinWidth = FMath::Min(ImageA->Width, ImageB->Width);
	const int32 MinHeight = FMath::Min(ImageA->Height, ImageB->Height);

	const int32 CompareWidth = FMath::Max(ImageA->Width, ImageB->Width);
	const int32 CompareHeight = FMath::Max(ImageA->Height, ImageB->Height);

	FImageDelta ImageDelta(CompareWidth, CompareHeight);

	volatile int32 MismatchCount = 0;

	// We create 100 blocks of local mismatch area, then bucket the pixel based on a spacial hash.
	int32 BlockSizeX = FMath::RoundFromZero(CompareWidth / 10.0);
	int32 BlockSizeY = FMath::RoundFromZero(CompareHeight / 10.0);
	volatile int32 LocalMismatches[100];
	FPlatformMemory::Memzero((void*)&LocalMismatches, 100 * sizeof(int32));

	ParallelFor(CompareWidth,
		[&] (int32 ColumnIndex)
	{
		for ( int Y = 0; Y < CompareHeight; Y++ )
		{
			// If different sizes, fail comparisons outside the bounds of the smaller image
			if ( ColumnIndex >= MinWidth || Y >= MinHeight )
			{
				ImageDelta.SetErrorPixel(ColumnIndex, Y, FColor(255, 0, 0, 255));
				FPlatformAtomics::InterlockedIncrement(&MismatchCount);
				int32 SpacialHash = ( ( Y / BlockSizeY ) * 10 + ( ColumnIndex / BlockSizeX ) );
				FPlatformAtomics::InterlockedIncrement(&LocalMismatches[SpacialHash]);
				continue;
			}

			FColor PixelA = ImageA->GetPixel(ColumnIndex, Y);
			FColor PixelB = ImageB->GetPixel(ColumnIndex, Y);

			if ( Tolerance.IgnoreColors )
			{
				if ( FPixelOperations::IsBrightnessSimilar(PixelA, PixelB, Tolerance) )
				{
					ImageDelta.SetClearPixel(ColumnIndex, Y);
				}
				else
				{
					ImageDelta.SetErrorPixel(ColumnIndex, Y);
					FPlatformAtomics::InterlockedIncrement(&MismatchCount);
					int32 SpacialHash = ( (Y / BlockSizeY) * 10 + ( ColumnIndex / BlockSizeX ) );
					FPlatformAtomics::InterlockedIncrement(&LocalMismatches[SpacialHash]);
				}

				// Next Pixel
				continue;
			}

			if ( FPixelOperations::IsRGBSimilar(PixelA, PixelB, Tolerance) )
			{
				ImageDelta.SetClearPixel(ColumnIndex, Y);
			}
			else if ( Tolerance.IgnoreAntiAliasing && (
				FPixelOperations::IsAntialiased(PixelA, ImageA, ColumnIndex, Y, Tolerance) ||
				FPixelOperations::IsAntialiased(PixelB, ImageB, ColumnIndex, Y, Tolerance)
				) )
			{
				if ( FPixelOperations::IsBrightnessSimilar(PixelA, PixelB, Tolerance) )
				{
					ImageDelta.SetClearPixel(ColumnIndex, Y);
				}
				else
				{
					ImageDelta.SetErrorPixel(ColumnIndex, Y);
					FPlatformAtomics::InterlockedIncrement(&MismatchCount);
					int32 SpacialHash = ( ( Y / BlockSizeY ) * 10 + ( ColumnIndex / BlockSizeX ) );
					FPlatformAtomics::InterlockedIncrement(&LocalMismatches[SpacialHash]);
				}
			}
			else
			{
				ImageDelta.SetErrorPixel(ColumnIndex, Y);
				FPlatformAtomics::InterlockedIncrement(&MismatchCount);
				int32 SpacialHash = ( ( Y / BlockSizeY ) * 10 + ( ColumnIndex / BlockSizeX ) );
				FPlatformAtomics::InterlockedIncrement(&LocalMismatches[SpacialHash]);
			}
		}
	});

	if (!OutDeltaPath.IsEmpty())
	{
		ImageDelta.Save(OutDeltaPath);
	}

	int32 MaximumLocalMismatches = 0;
	for ( int32 SpacialIndex = 0; SpacialIndex < 100; SpacialIndex++ )
	{
		MaximumLocalMismatches = FMath::Max(MaximumLocalMismatches, LocalMismatches[SpacialIndex]);
	}

	Results.Tolerance = Tolerance;
	Results.MaxLocalDifference = MaximumLocalMismatches / (double)( BlockSizeX * BlockSizeY );
	Results.GlobalDifference = MismatchCount / (double)( CompareHeight * CompareWidth );
	Results.ComparisonFilePath = ImageDelta.OutputComparisonFile;
	Results.CreationTime = FDateTime::Now();

	// In the case of differently sized images we force a failure
	if ( ImageA->Width != ImageB->Width || ImageA->Height != ImageB->Height )
	{
		Results.ErrorMessage = FText::FormatNamed(LOCTEXT("DifferentImageSizes", "Image comparison failed as sizes do not match, {WidthA}x{HeightA} vs {WidthB}x{HeightB}"),
			TEXT("WidthA"), ImageA->Width, TEXT("HeightA"), ImageA->Height,
			TEXT("WidthB"), ImageB->Width, TEXT("HeightB"), ImageB->Height);

		Results.Tolerance = Tolerance;
		Results.MaxLocalDifference = 1.0f;
		Results.GlobalDifference = 1.0f;
	}

	return Results;
}

double FImageComparer::CompareStructuralSimilarity(const FString& ImagePathA, const FString& ImagePathB, EStructuralSimilarityComponent InCompareComponent, const FString& OutDeltaPath)
{
	FImageComparisonResult Results;
	Results.ApprovedFilePath = ImagePathA;
	Results.IncomingFilePath = ImagePathB;

	FText ErrorA, ErrorB;
	FComparableImage ImageA, ImageB;

	if ( !ImageA.LoadFile(ImagePathA, ErrorA) )
	{
		Results.ErrorMessage = ErrorA;
		return 0.0f;
	}

	if ( !ImageB.LoadFile(ImagePathB, ErrorB) )
	{
		Results.ErrorMessage = ErrorB;
		return 0.0f;
	}

	if ( ImageA.Width != ImageB.Width || ImageA.Height != ImageB.Height )
	{
		Results.ErrorMessage = LOCTEXT("DifferentSizesUnsupported", "We can not compare images of different sizes at this time.");
		return 0.0f;
	}

	//ImageA->Process();
	//ImageB->Process();

	return CompareStructuralSimilarity(&ImageA, &ImageB, InCompareComponent, OutDeltaPath);
}

double FImageComparer::CompareStructuralSimilarity(const FComparableImage* ImageA, const FComparableImage* ImageB, EStructuralSimilarityComponent InCompareComponent, const FString& OutDeltaPath)
{
	// Implementation of https://en.wikipedia.org/wiki/Structural_similarity

	const double K1 = 0.01;
	const double K2 = 0.03;

	const int32 BitsPerComponent = 8;

	const int32 MaxWindowSize = 8;

	const int32 ImageWidth = ImageA->Width;
	const int32 ImageHeight = ImageA->Height;

	int32 TotalWindows = 0;
	double TotalSSIM = 0;
	
	FImageDelta ImageDelta(ImageWidth, ImageHeight);

	for ( int32 X = 0; X < ImageWidth; X += MaxWindowSize )
	{
		for ( int32 Y = 0; Y < ImageHeight; Y += MaxWindowSize )
		{
			const int32 WindowWidth = FMath::Min(MaxWindowSize, ImageWidth - X);
			const int32 WindowHeight = FMath::Min(MaxWindowSize, ImageHeight - Y);
			const int32 WindowEndX = X + WindowWidth;
			const int32 WindowEndY = Y + WindowHeight;

			double AverageA = 0;
			double AverageB = 0;
			double VarianceA = 0;
			double VarianceB = 0;
			double CovarianceAB = 0;

			TArray<double> ComponentA;
			TArray<double> ComponentB;

			// Run through the window, accumulate an average and the components for the window.
			for ( int32 WindowX = X; WindowX < WindowEndX; WindowX++ )
			{
				for ( int32 WindowY = Y; WindowY < WindowEndY; WindowY++ )
				{
					if ( InCompareComponent == EStructuralSimilarityComponent::Luminance )
					{
						const double LuminanceA = FPixelOperations::GetLuminance(ImageA->GetPixel(WindowX, WindowY));
						const double LuminanceB = FPixelOperations::GetLuminance(ImageB->GetPixel(WindowX, WindowY));
						AverageA += LuminanceA;
						AverageB += LuminanceB;
						ComponentA.Add(LuminanceA);
						ComponentB.Add(LuminanceB);
					}
					else // EStructuralSimilarityComponent::Color
					{
						const FColor ColorA = ImageA->GetPixel(WindowX, WindowY);
						const FColor ColorB = ImageB->GetPixel(WindowX, WindowY);
						const double ColorLumpA = ( ColorA.R + ColorA.G + ColorA.B ) * ( ColorA.A / 255.0 );
						const double ColorLumpB = ( ColorB.R + ColorB.G + ColorB.B ) * ( ColorB.A / 255.0 );
						AverageA += ColorLumpA;
						AverageB += ColorLumpB;
						ComponentA.Add(ColorLumpA);
						ComponentB.Add(ColorLumpB);
					}
				}
			}

			int32 WindowComponentCount = WindowWidth * WindowHeight;

			// Finally calculate the average.
			AverageA /= (double)WindowComponentCount;
			AverageB /= (double)WindowComponentCount;

			check(ComponentA.Num() == WindowComponentCount);
			check(ComponentB.Num() == WindowComponentCount);

			// Compute the Variance of A and B
			for ( int32 ComponentIndex = 0; ComponentIndex < WindowComponentCount; ComponentIndex++ )
			{
				const double DifferenceA = ComponentA[ComponentIndex] - AverageA;
				const double DifferenceB = ComponentB[ComponentIndex] - AverageB;

				const double SquaredDifferenceA = FMath::Pow(DifferenceA, 2);
				const double SquaredDifferenceB = FMath::Pow(DifferenceB, 2);

				VarianceA += SquaredDifferenceA;
				VarianceB += SquaredDifferenceB;
				CovarianceAB += DifferenceA * DifferenceB;
			}

			// Finally divide by the number of components to get the average of the mean of the squared differences (aka variance).
			VarianceA /= (double)WindowComponentCount;
			VarianceB /= (double)WindowComponentCount;
			CovarianceAB /= (double)WindowComponentCount;

			// The dynamic range of the pixel values
			float L = ( 1 << BitsPerComponent ) - 1;

			// Two variables to stabilize the division of a weak denominator.
			double C1 = FMath::Pow(K1 * L, 2);
			double C2 = FMath::Pow(K2 * L, 2);

			double Luminance = ( 2 * AverageA * AverageB + C1 ) / ( FMath::Pow(AverageA, 2) + FMath::Pow(AverageB, 2) + C1 );
			double Contrast = ( 2 * CovarianceAB + C2) / ( VarianceA + VarianceB + C2 );

			double WindowSSIM = Luminance * Contrast;
			double WindowDSIM = (1 - FMath::Clamp(WindowSSIM, 0.0, 1.0)) / 2;
			auto Color = FColor(WindowDSIM, WindowDSIM, WindowDSIM);
			for (int i = 0; i < MaxWindowSize; ++i) for (int j = 0; j < MaxWindowSize; ++j) {
				ImageDelta.SetErrorPixel(X + i, Y + j, Color);
			}

			TotalSSIM += WindowSSIM;
			TotalWindows++;
		}
	}

	if (!OutDeltaPath.IsEmpty())
	{
		ImageDelta.Save(OutDeltaPath);
	}

	double SSIM = TotalSSIM / TotalWindows;
	
	return FMath::Clamp(SSIM, 0.0, 1.0);
}

#undef LOCTEXT_NAMESPACE

