// Copyright Epic Games, Inc. All Rights Reserved.

#include "ScreenShotManager.h"
#include "AutomationWorkerMessages.h"
#include "Async/ParallelFor.h"
#include "Async/Async.h"
#include "HAL/FileManager.h"
#include "MessageEndpointBuilder.h"
#include "Misc/FileHelper.h"
#include "Misc/Paths.h"
#include "Misc/EngineVersion.h"
#include "Misc/FilterCollection.h"
#include "Misc/ConfigCacheIni.h"
#include "Serialization/JsonReader.h"
#include "Serialization/JsonSerializer.h"
#include "JsonObjectConverter.h"
#include "PlatformInfo.h"
#include "Misc/DataDrivenPlatformInfoRegistry.h"
#include "ScreenShotComparisonSettings.h"
#include "ILauncherServicesModule.h"
#include "RenderUtils.h"

DEFINE_LOG_CATEGORY(LogScreenShotManager);

#define LOCTEXT_NAMESPACE "ScreenShotManager"

/* Return true of the Reference screenshot is compatible with the incoming one */
static bool IsCompatible(const FAutomationScreenshotMetadata& Reference, const FAutomationScreenshotMetadata& Incoming)
{
	// If Substrate is disabled and the screenshot was approved with Substrate enabled, mark it as incompatible as it might contains result specific to Substrate.
	return Substrate::IsSubstrateEnabled() ? true : !Reference.bIsSubstrate;
}

class FScreenshotComparisons
{
public:
	FString ApprovedFolder;
	FString UnapprovedFolder;

	TArray<FString> Existing;
	TArray<FString> New;
	TArray<FString> Missing;
};

FScreenShotManager::FScreenShotManager()
{
	FModuleManager::Get().LoadModuleChecked(FName("ImageWrapper"));

	ScreenshotResultsFolder = FPaths::AutomationReportsDir();
	ScreenshotTempDeltaFolder = FPaths::Combine(FPaths::AutomationTransientDir(), TEXT("Delta/"));

	const UScreenShotComparisonSettings* ScreenshotSettings = GetDefault<UScreenShotComparisonSettings>();
	bUseConfidentialPlatformPaths = ScreenshotSettings->bUseConfidentialPlatformPathsForSavedResults;

	// the automation controller owns the screenshot manager so we don't have to worry about outliving it
	//AutomationController->OnTestsAvailable().AddRaw(this, &FScreenShotManager::OnTestAvailableCallback);

	BuildFallbackPlatformsListFromConfig();
}

FScreenShotManager::~FScreenShotManager()
{
}



FString FScreenShotManager::GetPathComponentForRHI(const FAutomationScreenshotMetadata& MetaData) const
{
	FString RHIInfo = MetaData.Rhi;
	
	if (!RHIInfo.IsEmpty())
	{
		RHIInfo += TEXT("_");
	}

	RHIInfo += MetaData.FeatureLevel;

	return RHIInfo;
}


FString FScreenShotManager::GetPathComponentForPlatformAndRHI(const FAutomationScreenshotMetadata& MetaData) const
{
	return FPaths::Combine(MetaData.Platform, GetPathComponentForRHI(MetaData));
}

/**
 * Images are now preferred to be under MapOrContext/ImageName/Plat/RHI etc
 */
FString FScreenShotManager::GetPathComponentForTestImages(const FAutomationScreenshotMetadata& MetaData, bool bIncludeVariantName) const
{
	FString FilePath;
	if (bIncludeVariantName)
	{
		FilePath = FPaths::Combine(MetaData.Context, *MetaData.ScreenShotName, *MetaData.VariantName);
	}
	else
	{
		FilePath = FPaths::Combine(MetaData.Context, *MetaData.ScreenShotName);
	}
	FPaths::NormalizeDirectoryName(FilePath);
	return FilePath;
}

FString FScreenShotManager::GetApprovedFolderForImageWithOptions(const FAutomationScreenshotMetadata& MetaData, EApprovedFolderOptions InOptions) const
{
	const FDataDrivenPlatformInfo& PlatInfo = FDataDrivenPlatformInfoRegistry::GetPlatformInfo(MetaData.Platform);

	bool bUsePlatformPath = PlatInfo.bIsConfidential && (InOptions & EApprovedFolderOptions::UsePlatformFolders) != 0;

	// Test folder will be MapOrContext/ImageName
	FString TestFolder = GetPathComponentForTestImages(MetaData, false);

	FString OutPath = FPaths::ProjectDir();

#if !WITH_EDITOR
	// Project path would be different if project is started from Unreal Frontend (uses LauncherServices)
	if (FModuleManager::Get().IsModuleLoaded("LauncherServices"))
	{
		ILauncherServicesModule& LauncherServicesModule = FModuleManager::LoadModuleChecked<ILauncherServicesModule>("LauncherServices");
		FString ProjectPath = FPaths::GetPath(LauncherServicesModule.GetProfileManager()->GetProjectPath());
		OutPath = !ProjectPath.IsEmpty() && FPaths::DirectoryExists(ProjectPath) ? ProjectPath : OutPath;
	}
#endif

	if (bUsePlatformPath)
	{
		OutPath = FPaths::Combine(OutPath, TEXT("Platforms"), MetaData.Platform, TEXT("Test/Screenshots/"), TestFolder, GetPathComponentForRHI(MetaData));
	}
	else
	{
		OutPath = FPaths::Combine(OutPath, TEXT("Test/Screenshots/"), TestFolder, MetaData.Platform, GetPathComponentForRHI(MetaData));
	}

	if (!OutPath.EndsWith(TEXT("/")))
	{
		OutPath += TEXT("/");
	}

	return OutPath;
}

FString FScreenShotManager::GetIdealApprovedFolderForImage(const FAutomationScreenshotMetadata& MetaData) const
{
	EApprovedFolderOptions DefaultOptions = bUseConfidentialPlatformPaths ? EApprovedFolderOptions::UsePlatformFolders : EApprovedFolderOptions::None;
	return GetApprovedFolderForImageWithOptions(MetaData, DefaultOptions);
}

TArray<FString> FScreenShotManager::FindApprovedFiles(const FAutomationScreenshotMetadata& IncomingMetaData, const FString& FilePattern) const
{
	TArray<FString> TriedPaths;

	auto FindImages = [&TriedPaths, FilePattern](TArray<FString>& OutApprovedImages, const FString& InPath)
	{
		IFileManager::Get().FindFilesRecursive(OutApprovedImages, *InPath, *FilePattern, true, false);
		TriedPaths.Emplace(InPath);
	};

	TArray<FString> ApprovedImages;

	EApprovedFolderOptions Options = bUseConfidentialPlatformPaths ? EApprovedFolderOptions::UsePlatformFolders : EApprovedFolderOptions::None;

	// check out standard path using whether confidential platforms are in a separate tree
	FString ApprovedPath = GetApprovedFolderForImageWithOptions(IncomingMetaData, Options);
	FindImages(ApprovedImages, ApprovedPath);

	// Make sure the first log line is of the first path tried, not the last fallback. The list of fallbacks will be printed if nothing is found.
	const FString FirstApprovedPath = ApprovedPath;

	// if we're a blank and bUseConfidentialPlatformPaths, try without that
	if (ApprovedImages.Num() == 0 && bUseConfidentialPlatformPaths)
	{
		// check standard paths.
		ApprovedPath = FPaths::GetPath(GetApprovedFolderForImageWithOptions(IncomingMetaData, EApprovedFolderOptions::None));
		FindImages(ApprovedImages, ApprovedPath);
	}

	// find fallback images if they don't exist at this point
	if (ApprovedImages.Num() == 0)
	{
		FString CurrentPlatformRHI = GetPathComponentForPlatformAndRHI(IncomingMetaData);

		UE_LOGF(LogScreenShotManager, Log, "No ideal-image found at %ls. Checking fallback images", *FirstApprovedPath);

		while (ApprovedImages.Num() == 0)
		{
			const FString* FallbackPlatformRHI = FallbackPlatforms.Find(CurrentPlatformRHI);
			if (!FallbackPlatformRHI)
			{
				break;
			}

			CurrentPlatformRHI = *FallbackPlatformRHI;

			TArray<FString> Components;
			CurrentPlatformRHI.ParseIntoArray(Components, TEXT("/"));			

			if (Components.Num() == 2)
			{
				TArray<FString> FeatureLevels;
				Components[1].ParseIntoArray(FeatureLevels, TEXT("_"));

				FAutomationScreenshotMetadata CopiedMetaData = IncomingMetaData;
				CopiedMetaData.Platform = Components[0];
				if (FeatureLevels.Num() > 1)
				{
					CopiedMetaData.Rhi = FeatureLevels[0];
					CopiedMetaData.FeatureLevel = FeatureLevels[1];
				}
				else
				{
					// We don't need to do RHI_FL, just FL
					CopiedMetaData.Rhi = TEXT("");
					CopiedMetaData.FeatureLevel = FeatureLevels[0];
				}

				ApprovedPath = FPaths::GetPath(GetIdealApprovedFolderForImage(CopiedMetaData));
				FindImages(ApprovedImages, ApprovedPath);

				if (ApprovedImages.Num())
				{
					UE_LOGF(LogScreenShotManager, Log, "Using fallback images from %ls", *ApprovedPath);
				}
			}
			else
			{
				UE_LOGF(LogScreenShotManager, Error, "Invalid fallback Platform/RHI string %ls", *CurrentPlatformRHI);
			}	
		}
	}

	if (ApprovedImages.IsEmpty())
	{
		UE_LOGF(LogScreenShotManager, Log, "Couldn't Find any fallback images, tried the following paths:");
		for (const FString& TriedPath : TriedPaths)
		{
			UE_LOGF(LogScreenShotManager, Log, "    %ls", *TriedPath);
		}
	}

	return ApprovedImages;
}

/* IScreenShotManager event handlers
 *****************************************************************************/

TFuture<FImageComparisonResult> FScreenShotManager::CompareScreenshotAsync(const FString& IncomingPath, const FAutomationScreenshotMetadata& MetaData, const EScreenShotCompareOptions Options)
{
	return Async(EAsyncExecution::Thread, [this, IncomingPath, MetaData, Options] () { return CompareScreenshot(IncomingPath, MetaData, Options); });
}

/**
 * Takes an incoming file and returns a comparison based on comparing it to a reference image that exists in the project.
 * The comparison results (incoming image, approved image, comparison image, report metadata) will be placed in the provided
 * output folder.
 *
 * The incoming path must contain a recognized platform name as a path component, with the feature-level info of that platform
 * as the next component.
 * 
 * E.g. <path> must contain <early-path>/Windows/D3D12_SM5/<later-path>, or <early-path>/PS4/SM5/<later-path> etc and must exist
 * under the path returned by IScreenshotManager::GetIncomingFolder
 *
 * Comparison images are loaded from <project>/Test/<path> unless ScreenShotComparisonSettings.bUseConfidentialPlatformPathsForSavedResults 
 * is set in which case they will come from <project>/Platforms/<platform>/Test
 * 
 * @param InUnapprovedIncomingFilePath Path
 * @param ResultsSubFolder 
 *
 * @return FImageComparisonResult
 */
FImageComparisonResult FScreenShotManager::CompareScreenshot(const FString& InUnapprovedIncomingFilePath, const FAutomationScreenshotMetadata& IncomingMetaData, const EScreenShotCompareOptions Options)
{
	// #agrant todo: Handle bad paths

	// get the ideal path for our approved image. This is the path that a file would be at if it matches our platform and RHI
	FString IdealApprovedFolderPath = GetIdealApprovedFolderForImage(IncomingMetaData);

	FString ResultsSubFolder = GetPathComponentForTestImages(IncomingMetaData, true);

	// If the metadata for the screenshot does not provide tolerance rules, use these instead.
	FImageTolerance DefaultTolerance = FImageTolerance::DefaultIgnoreLess;
	DefaultTolerance.IgnoreAntiAliasing = true;

	// find all the approved images we can use. This will find fallback images from other platforms if necessary
	TArray<FString> ApprovedDeviceShots = FindApprovedFiles(IncomingMetaData, TEXT("*.png"));

	TOptional<FAutomationScreenshotMetadata> NearestExistingApprovedImageMetadata;

	// This is copied over later, so don't set any properties...
	FImageComparisonResult ComparisonResult;

	// Use found shots as ground truth
	if (ApprovedDeviceShots.Num() > 0)
	{	
		TOptional<FAutomationScreenshotMetadata> ExistingMetadata;
		{
			// Always read the metadata file from the unapproved location, as we may have introduced new comparison rules.
			FString MetadataFile = FPaths::ChangeExtension(InUnapprovedIncomingFilePath, ".json");

			FString Json;
			if ( FFileHelper::LoadFileToString(Json, *MetadataFile) )
			{
				FAutomationScreenshotMetadata Metadata;
				if ( FJsonObjectConverter::JsonObjectStringToUStruct(Json, &Metadata, 0, 0) )
				{
					ExistingMetadata = Metadata;
				}
			}
		}

		FString NearestExistingApprovedImage;
		if ( ExistingMetadata.IsSet() )
		{
			int32 MatchScore = -1;

			for ( FString ApprovedShot : ApprovedDeviceShots )
			{	
				FString ApprovedMetadataFile = FPaths::ChangeExtension(ApprovedShot, ".json");

				FString Json;
				if ( FFileHelper::LoadFileToString(Json, *ApprovedMetadataFile) )
				{
					FAutomationScreenshotMetadata Metadata;
					if ( FJsonObjectConverter::JsonObjectStringToUStruct(Json, &Metadata, 0, 0) )
					{
						// If there is only one reference screenshot, use it.
						const bool bCompatibleScreenShot = ApprovedDeviceShots.Num() == 1 || IsCompatible(Metadata, ExistingMetadata.GetValue());
						if (!bCompatibleScreenShot)
						{
							continue;
						}

						int32 Comparison = Metadata.Compare(ExistingMetadata.GetValue());
						if ( Comparison > MatchScore )
						{
							MatchScore = Comparison;
							NearestExistingApprovedImage = ApprovedShot;
							NearestExistingApprovedImageMetadata = Metadata;
						}
					}
				}
			}
		}
		else
		{
			// TODO no metadata how do I pick a good shot?
			NearestExistingApprovedImage = ApprovedDeviceShots[0];
		}

		FImageTolerance Tolerance = DefaultTolerance;

		if ( ExistingMetadata.IsSet() && ExistingMetadata->bHasComparisonRules )
		{
			Tolerance.Red = ExistingMetadata->ToleranceRed;
			Tolerance.Green = ExistingMetadata->ToleranceGreen;
			Tolerance.Blue = ExistingMetadata->ToleranceBlue;
			Tolerance.Alpha = ExistingMetadata->ToleranceAlpha;
			Tolerance.MinBrightness = ExistingMetadata->ToleranceMinBrightness;
			Tolerance.MaxBrightness = ExistingMetadata->ToleranceMaxBrightness;
			Tolerance.IgnoreAntiAliasing = ExistingMetadata->bIgnoreAntiAliasing;
			Tolerance.IgnoreColors = ExistingMetadata->bIgnoreColors;
			Tolerance.MaximumLocalError = ExistingMetadata->MaximumLocalError;
			Tolerance.MaximumGlobalError = ExistingMetadata->MaximumGlobalError;
		}

		// TODO Think about using SSIM, but needs local SSIM as well as Global SSIM, same as the basic comparison.
		//double SSIM = Comparer.CompareStructuralSimilarity(ApprovedFullPath, UnapprovedFullPath, FImageComparer::EStructuralSimilarityComponent::Luminance);
		//printf("%f\n", SSIM);

		FImageComparer Comparer;

		ComparisonResult = Comparer.Compare(NearestExistingApprovedImage, InUnapprovedIncomingFilePath, Tolerance, ScreenshotTempDeltaFolder);
	}
	else
	{
		// We can't find a ground truth, so it's a new comparison.
		ComparisonResult.IncomingFilePath = InUnapprovedIncomingFilePath;
		ComparisonResult.CreationTime = FDateTime::UtcNow();

		UE_LOGF(LogScreenShotManager, Log, "No ideal-image found. Assuming %ls is a new test image", *InUnapprovedIncomingFilePath);
	}

	ComparisonResult.SourcePlatform = IncomingMetaData.Platform;
	ComparisonResult.SourceRHI = IncomingMetaData.Rhi;
	ComparisonResult.IdealApprovedFolderPath = IdealApprovedFolderPath;
	// We use the subfolder path to the screenshot as name (before any environment specialization - platform, RHI - are appended).
	ComparisonResult.ScreenshotPath = ResultsSubFolder;
	ComparisonResult.ComparisonId = IncomingMetaData.Id;

	// Do not save passing variant test screenshots
	// Disabled for now until more variants are added, since variants now run without baseline tests in lightweight mode and we need to save at least one screenshot
	/*const bool bIsVariant = !IncomingMetaData.VariantName.IsEmpty();
	if (bIsVariant && FAutomationTestFramework::Get().NeedUseLightweightStereoTestVariants() && ComparisonResult.AreSimilar())
	{
		ComparisonResult.bSkipAttachingImages = true;
		return ComparisonResult;
	}*/

	// Result paths should be relative to the project. Note this may be empty, and if it is MakePathRelative returns
	// a non empty relative path... but we want it to stay empty as that's how we signal that no approved file exists

	// these two must exist...
	FPaths::MakePathRelativeTo(ComparisonResult.IncomingFilePath, *FPaths::ProjectDir());
	FPaths::MakePathRelativeTo(ComparisonResult.IdealApprovedFolderPath, *FPaths::ProjectDir());

	if (!ComparisonResult.ApprovedFilePath.IsEmpty())
	{
		FPaths::MakePathRelativeTo(ComparisonResult.ApprovedFilePath, *FPaths::ProjectDir());
	}

	if (!ComparisonResult.ComparisonFilePath.IsEmpty())
	{
		FPaths::MakePathRelativeTo(ComparisonResult.ComparisonFilePath, *FPaths::ProjectDir());
	}

	// Report path is something like Test/Context/ that can be freely moved around and relocated with the relative paths
	// to the data remaining intact
	FString ReportPathOnDisk = FPaths::Combine(ScreenshotResultsFolder, ResultsSubFolder, IncomingMetaData.Platform, GetPathComponentForRHI(IncomingMetaData), TEXT("/"));

	/*
		Now create copies of all three images for the report. We use copies (a move in the case of the delta image) so that the report folders
		can be moved around without any external references. This is vital for CIS where we want the results to be served up via HTTP or just 
		by browsing without needing the matching project directory from the time or any other saved data.
	*/
	
	// files we write/copy to for reports
	ComparisonResult.ReportApprovedFilePath = FPaths::Combine(ReportPathOnDisk, TEXT("Approved.png")); 
	ComparisonResult.ReportIncomingFilePath = FPaths::Combine(ReportPathOnDisk, TEXT("Incoming.png"));
	ComparisonResult.ReportComparisonFilePath = FPaths::Combine(ReportPathOnDisk, TEXT("Delta.png"));

	// If not ideal the make the approved name contain the source for clarity, since it will be under a Plat/RHI/Feature level folder for the platform being tested
	if (!ComparisonResult.IsIdeal() && !ComparisonResult.IsNew())
	{
		check(NearestExistingApprovedImageMetadata.IsSet());
		FString ApprovedName = FString::Printf(TEXT("Approved_%s.png"), *GetPathComponentForPlatformAndRHI(*NearestExistingApprovedImageMetadata).Replace(TEXT("/"), TEXT("_")));
		ComparisonResult.ReportApprovedFilePath = FPaths::Combine(ReportPathOnDisk, ApprovedName);
	}

	const FString ProjectDir = FPaths::GetPath(FPaths::IsProjectFilePathSet() ? FPaths::GetProjectFilePath() : FPaths::ProjectDir());

	/*
		First we need the incoming file in the report. If the calling code wants to keep the image then do a copy, 
		otherwise move it
	*/
	// #agrant todo - code in AutomationControllerManager requires these paths and at the moment does not have enough info to use the report versions. Need to also change
	// the delta image below back to a move as well
//	bool CanMoveImage = false; // Options != EScreenShotCompareOptions::KeepImage
//	if (CanMoveImage == false)
	{
		// copy the incoming file to the report path 
		const FString IncomingFileFullPath = *FPaths::Combine(ProjectDir, ComparisonResult.IncomingFilePath);

		if (IFileManager::Get().Copy(*ComparisonResult.ReportIncomingFilePath, *IncomingFileFullPath, true, true) == COPY_OK)
		{
			IFileManager::Get().Copy(*FPaths::ChangeExtension(ComparisonResult.ReportIncomingFilePath, ".json"), *FPaths::ChangeExtension(IncomingFileFullPath, ".json"), true, true);
		}
		else
		{
			UE_LOGF(LogScreenShotManager, Error, "Failed to copy incoming image to %ls", *ComparisonResult.ReportApprovedFilePath);
		}
	}
/*	else
	{
		// we can discard the incoming image, so just move it...
		FString IncomingFileFullPath = *FPaths::Combine(ProjectDir, ComparisonResult.IncomingFilePath);

		if (IFileManager::Get().Move(*ComparisonResult.ReportIncomingFilePath, *IncomingFileFullPath, true, true))
		{
			IFileManager::Get().Move(*FPaths::ChangeExtension(ComparisonResult.ReportIncomingFilePath, ".json"), *FPaths::ChangeExtension(IncomingFileFullPath, ".json"), true, true);
		}
		else
		{
			UE_LOGF(LogScreenShotManager, Error, "Failed to copy incoming image to %ls", *ComparisonResult.ReportApprovedFilePath);
		}
	}*/

	// move any incoming renderdoc file if one existed
	FString IncomingTraceFile = FPaths::ChangeExtension(InUnapprovedIncomingFilePath, TEXT(".rdc"));
	if (IFileManager::Get().FileExists(*IncomingTraceFile))
	{
		IFileManager::Get().Move(*(FPaths::Combine(ReportPathOnDisk, TEXT("Incoming.rdc"))), *IncomingTraceFile, true, true);
	}

	// copy the comparison image if we generated one
	if (ComparisonResult.ComparisonFilePath.Len())
	{
		FString ComparisonFileFullPath = *FPaths::Combine(ProjectDir, ComparisonResult.ComparisonFilePath);

		if (IFileManager::Get().Copy(*ComparisonResult.ReportComparisonFilePath, *ComparisonFileFullPath, true, true) == COPY_OK)
		{
			// nothing else to move for this case
		}
		else
		{
			UE_LOGF(LogScreenShotManager, Error, "Failed to move delta image to %ls", *ComparisonResult.ReportComparisonFilePath);
		}
	}

	/*
		Make a copy of the approved file if it exists
	*/
	if (ComparisonResult.ApprovedFilePath.Len())
	{
		// Make a copy of the approved png called 'approved.png'
		FString ApprovedFileFullPath = *FPaths::Combine(ProjectDir, ComparisonResult.ApprovedFilePath);

		if (IFileManager::Get().Copy(*ComparisonResult.ReportApprovedFilePath, *ApprovedFileFullPath, true, true) == COPY_OK)
		{
			IFileManager::Get().Copy(*FPaths::ChangeExtension(ComparisonResult.ReportApprovedFilePath, ".json"), *FPaths::ChangeExtension(ApprovedFileFullPath, ".json"), true, true);
		}
		else
		{
			UE_LOGF(LogScreenShotManager, Error, "Failed to copy approved image to %ls", *ComparisonResult.ReportApprovedFilePath);
		}
	}
		
	// make all these paths relative to the report path so the report can be relocated outside of the project directory
	// and the files still found.
	FPaths::MakePathRelativeTo(ComparisonResult.ReportApprovedFilePath, *ScreenshotResultsFolder);
	FPaths::MakePathRelativeTo(ComparisonResult.ReportIncomingFilePath, *ScreenshotResultsFolder);
	FPaths::MakePathRelativeTo(ComparisonResult.ReportComparisonFilePath, *ScreenshotResultsFolder);

	// save the result at to a report
	FString Json;
	if ( FJsonObjectConverter::UStructToJsonObjectString(ComparisonResult, Json) )
	{
		FString ComparisonReportFile = FPaths::Combine(ReportPathOnDisk, TEXT("Report.json"));
		FFileHelper::SaveStringToFile(Json, *ComparisonReportFile, FFileHelper::EEncodingOptions::ForceUTF8WithoutBOM);

		UE_LOGF(LogScreenShotManager, Log, "Saved report to %ls", *ComparisonReportFile);
	}

	return ComparisonResult;
}


FScreenshotExportResult FScreenShotManager::ExportScreenshotComparisonResult(FString ScreenshotName, FString RootExportFolder, bool bOnlyGeneratedFiles)
{
	FPaths::NormalizeDirectoryName(RootExportFolder);

	if (RootExportFolder.IsEmpty())
	{
		RootExportFolder = GetDefaultExportDirectory();
	}

	FScreenshotExportResult Results;
	Results.Success = false;
	Results.ExportPath = RootExportFolder;

	FString Destination = RootExportFolder / ScreenshotName;
	if (!IFileManager::Get().MakeDirectory(*Destination, /*Tree =*/true))
	{
		return Results;
	}

	TArray<FString> Patterns;
	if (bOnlyGeneratedFiles)
	{
		Patterns.Add(TEXT("Incoming.*"));
		Patterns.Add(TEXT("Delta.png"));
		Patterns.Add(TEXT("Report.json"));
	}
	else
	{
		Patterns.Add(TEXT("*"));
	}

	CopyDirectory(Destination, ScreenshotResultsFolder / ScreenshotName, Patterns);

	Results.Success = true;
	return Results;
}

TFuture<TSharedPtr<TArray<FComparisonReport>>> FScreenShotManager::OpenComparisonReportsAsync(const FString& ImportPath)
{
	FString PreprocessedImportPath(ImportPath);
	FPaths::NormalizeDirectoryName(PreprocessedImportPath);
	PreprocessedImportPath += TEXT("/");

	TArray<FString> ComparisonReportPaths;
	IFileManager::Get().FindFilesRecursive(ComparisonReportPaths, *PreprocessedImportPath, TEXT("Report.json"), true, false, false);
	
	// Note that if current PendingComparisonReportPaths is valid,
	// it will be reset in the line below (it will cancel the corresponding loading task)
	PendingComparisonReportPaths = MakeShared<TArray<FString>>(MoveTemp(ComparisonReportPaths));

	return Async(EAsyncExecution::ThreadPool, [this, ImportPathToUse = MoveTemp(PreprocessedImportPath), ReportPathsWPtr = PendingComparisonReportPaths.ToWeakPtr()]() -> TSharedPtr<TArray<FComparisonReport>>
	{
		int32 ReportPathsNum = 0;
		{
			TSharedPtr<TArray<FString>> ReportPathsSPtr = ReportPathsWPtr.Pin();
			if (ReportPathsSPtr.IsValid())
			{
				ReportPathsNum = ReportPathsSPtr->Num();
			}
			else
			{
				// The reports paths list is outdated. Cancel the job with default return value.
				return nullptr;
			}
		}

		if (ReportPathsNum <= 0)
		{
			return nullptr;
		}

		TSharedPtr<TArray<FComparisonReport>> ComparisonReports = MakeShared<TArray<FComparisonReport>>();
		ComparisonReports->Reserve(ReportPathsNum);

		for (int32 ReportPathIndex = 0; ReportPathIndex < ReportPathsNum; ++ReportPathIndex)
		{
			FString ReportPath;
			{
				TSharedPtr<TArray<FString>> ReportPathsSPtr = ReportPathsWPtr.Pin();
				if (ReportPathsSPtr.IsValid())
				{
					ReportPath = (*ReportPathsSPtr)[ReportPathIndex];
					check(!ReportPath.IsEmpty());
				}
				else
				{
					// The reports paths list is outdated. Cancel the job with default return value.
					return nullptr;
				}
			}

			FString JsonString;
			if (FFileHelper::LoadFileToString(JsonString, *ReportPath))
			{
				TSharedRef<TJsonReader<>> JsonReader = TJsonReaderFactory<>::Create(JsonString);

				TSharedPtr<FJsonObject> JsonComparisonReport;
				if (!FJsonSerializer::Deserialize(JsonReader, JsonComparisonReport))
				{
					return nullptr;
				}

				FImageComparisonResult ComparisonResult;
				ComparisonResult.SetInvalid();
				if (FJsonObjectConverter::JsonObjectToUStruct(JsonComparisonReport.ToSharedRef(), &ComparisonResult, 0, 0))
				{
					if (ComparisonResult.IsValid())
					{
						FComparisonReport Report = FComparisonReport(ImportPathToUse, ReportPath);
						Report.SetComparisonResult(ComparisonResult);
						ComparisonReports->Add(Report);
					}
					else
					{
						UE_LOGF(LogScreenShotManager, Error, "Report %ls has invalid version '%d' (Current Version=%d)", *ReportPath, ComparisonResult.Version, int32(ComparisonResult.CurrentVersion));
					}
				}
			}
		}

		return ComparisonReports;
	});
}

TSharedPtr<FImageComparisonResult> FScreenShotManager::CompareImageSequence(const TMap<FString, FString>& Sequence, const FAutomationScreenshotMetadata& Metadata)
{
	// Time for the computationally expensive part, doing image comparisons!
	for (const TPair<FString, FString>& Pair : Sequence)
	{
		const FString& OldImagePath = Pair.Key;
		const FString& NewImagePath = Pair.Value;

		// Calculate a path for the delta image to be saved.
		FString DeltaPath = FPaths::ChangeExtension(NewImagePath, TEXT(""));
		FString OldExtension = FPaths::GetExtension(NewImagePath, true);
		DeltaPath += TEXT("_Delta") + OldExtension;

		// If the metadata for the screenshot does not provide tolerance rules, use these instead.
		const FImageTolerance DefaultTolerance = FImageTolerance::DefaultIgnoreLess;
		FImageComparer Comparer;
		FImageTolerance Tolerance = DefaultTolerance;

		if (Metadata.bHasComparisonRules)
		{
			Tolerance.Red = Metadata.ToleranceRed;
			Tolerance.Green = Metadata.ToleranceGreen;
			Tolerance.Blue = Metadata.ToleranceBlue;
			Tolerance.Alpha = Metadata.ToleranceAlpha;
			Tolerance.MinBrightness = Metadata.ToleranceMinBrightness;
			Tolerance.MaxBrightness = Metadata.ToleranceMaxBrightness;
			Tolerance.IgnoreAntiAliasing = Metadata.bIgnoreAntiAliasing;
			Tolerance.IgnoreColors = Metadata.bIgnoreColors;
			Tolerance.MaximumLocalError = Metadata.MaximumLocalError;
			Tolerance.MaximumGlobalError = Metadata.MaximumGlobalError;
		}

		FImageComparisonResult ComparisonResult = Comparer.Compare(OldImagePath, NewImagePath, Tolerance, DeltaPath);

		if (!ComparisonResult.AreSimilar())
		{
			ComparisonResult.SourcePlatform = Metadata.Platform;
			ComparisonResult.SourceRHI = Metadata.Rhi;
			ComparisonResult.IdealApprovedFolderPath = GetIdealApprovedFolderForImage(Metadata);;
			ComparisonResult.ScreenshotPath = GetPathComponentForTestImages(Metadata, true);
			FString ReportPathOnDisk = FPaths::Combine(ScreenshotResultsFolder, GetPathComponentForTestImages(Metadata, true), Metadata.Platform, GetPathComponentForRHI(Metadata), TEXT("/"));

			ComparisonResult.ReportApprovedFilePath = FPaths::Combine(ReportPathOnDisk, TEXT("Approved.png"));
			ComparisonResult.ReportIncomingFilePath = FPaths::Combine(ReportPathOnDisk, TEXT("Incoming.png"));
			ComparisonResult.ReportComparisonFilePath = FPaths::Combine(ReportPathOnDisk, TEXT("Delta.png"));

			TArray<TPair<FString, FString>> FilesToCopy;
			FilesToCopy.Emplace(ComparisonResult.ApprovedFilePath, FPaths::ConvertRelativePathToFull(ComparisonResult.ReportApprovedFilePath));
			FilesToCopy.Emplace(ComparisonResult.IncomingFilePath, FPaths::ConvertRelativePathToFull(ComparisonResult.ReportIncomingFilePath));
			FilesToCopy.Emplace(ComparisonResult.ComparisonFilePath, FPaths::ConvertRelativePathToFull(ComparisonResult.ReportComparisonFilePath));

			ParallelFor(FilesToCopy.Num(), [&](int32 Index)
				{
					const TPair<FString, FString>& FilePathPair = FilesToCopy[Index];
					IFileManager::Get().Copy(*FilePathPair.Value, *FilePathPair.Key, true, true);
				});

			FPaths::MakePathRelativeTo(ComparisonResult.ReportApprovedFilePath, *ScreenshotResultsFolder);
			FPaths::MakePathRelativeTo(ComparisonResult.ReportIncomingFilePath, *ScreenshotResultsFolder);
			FPaths::MakePathRelativeTo(ComparisonResult.ReportComparisonFilePath, *ScreenshotResultsFolder);
			FPaths::MakePathRelativeTo(ComparisonResult.IncomingFilePath, *FPaths::ProjectDir());
			FPaths::MakePathRelativeTo(ComparisonResult.IdealApprovedFolderPath, *FPaths::ProjectDir());

			if (!ComparisonResult.ApprovedFilePath.IsEmpty())
			{
				FPaths::MakePathRelativeTo(ComparisonResult.ApprovedFilePath, *FPaths::ProjectDir());
			}
			if (!ComparisonResult.ComparisonFilePath.IsEmpty())
			{
				FPaths::MakePathRelativeTo(ComparisonResult.ComparisonFilePath, *FPaths::ProjectDir());
			}

			if (ComparisonResult.ErrorMessage.IsEmpty())
			{
				ComparisonResult.ErrorMessage = FText::Format(
					LOCTEXT("CompareImageSequenceError", "Frame '{0}' is not similar to Ground Truth"),
					FText::FromString(FPaths::GetBaseFilename(ComparisonResult.ApprovedFilePath))
				);
			}

			// save the result at to a report
			FString Json;
			if (FJsonObjectConverter::UStructToJsonObjectString(ComparisonResult, Json))
			{
				FString ComparisonReportFile = FPaths::Combine(ReportPathOnDisk, TEXT("Report.json"));
				FFileHelper::SaveStringToFile(Json, *ComparisonReportFile, FFileHelper::EEncodingOptions::ForceUTF8WithoutBOM);

				UE_LOGF(LogScreenShotManager, Log, "Saved report to %ls", *ComparisonReportFile);
			}
			if (FJsonObjectConverter::UStructToJsonObjectString(Metadata, Json))
			{
				FString MetadataPath = FPaths::Combine(ReportPathOnDisk, TEXT("Incoming.json"));
				FFileHelper::SaveStringToFile(Json, *MetadataPath, FFileHelper::EEncodingOptions::ForceUTF8WithoutBOM);
			}

			return MakeShareable(new FImageComparisonResult(ComparisonResult));
		}
	}

	return nullptr;
}

void FScreenShotManager::NotifyAutomationTestFrameworkOfImageComparison(const FImageComparisonResult& ComparisonResult)
{
	FAutomationScreenshotCompareResults Results(
		FGuid::NewGuid(),
		ComparisonResult.ErrorMessage.ToString(),
		ComparisonResult.MaxLocalDifference,
		ComparisonResult.GlobalDifference,
		ComparisonResult.IsNew(),
		ComparisonResult.AreSimilar(),
		ComparisonResult.IncomingFilePath,
		ComparisonResult.ReportComparisonFilePath,
		ComparisonResult.ReportApprovedFilePath,
		ComparisonResult.ReportIncomingFilePath,
		ComparisonResult.ScreenshotPath
	);
	FAutomationTestFramework::Get().NotifyScreenshotComparisonReport(Results);
	if (FAutomationTestBase* CurrentTest = FAutomationTestFramework::Get().GetCurrentTest())
	{
		CurrentTest->AddEvent(Results.ToAutomationEvent());
	}
}

FString FScreenShotManager::GetDefaultExportDirectory() const
{
	return FPaths::Combine(FPaths::ProjectSavedDir(),TEXT("Exported/imageCompare"));
}

void FScreenShotManager::CopyDirectory(const FString& DestDir, const FString& SrcDir, const TArray<FString> Patterns)
{
	TArray<FString> FilesToCopy;

	FString AbsoluteSrcDir = FPaths::ConvertRelativePathToFull(SrcDir);

	for (auto& Pattern : Patterns)
	{
		IFileManager::Get().FindFilesRecursive(FilesToCopy, *AbsoluteSrcDir, *Pattern, /*Files=*/true, /*Directories=*/false, /*bClearFileNames=*/false );
	}

	ParallelFor(FilesToCopy.Num(), [&](int32 Index)
		{
			const FString& SourceFilePath = FilesToCopy[Index];
			FString DestFilePath = FPaths::Combine(DestDir, SourceFilePath.RightChop(AbsoluteSrcDir.Len()));
			IFileManager::Get().Copy(*DestFilePath, *SourceFilePath, true, true);
		});
}

void FScreenShotManager::BuildFallbackPlatformsListFromConfig()
{
	FallbackPlatforms.Empty();

	const UScreenShotComparisonSettings* ScreenshotSettings = GetDefault<UScreenShotComparisonSettings>();
	TSet<FScreenshotFallbackEntry> ScreenshotFallbackPlatforms(ScreenshotSettings->ScreenshotFallbackPlatforms);
#if WITH_EDITOR
	ScreenshotFallbackPlatforms.Append(UScreenShotComparisonSettings::GetAllPlatformSettings());
#endif // WITH_EDITOR

	if (ScreenshotFallbackPlatforms.Num() > 0)
	{
		for (const FScreenshotFallbackEntry& Entry : ScreenshotFallbackPlatforms)
		{
			FString Parent = Entry.Parent;
			FString Child = Entry.Child;

			if (Parent.StartsWith(TEXT("/")))
			{
				UE_LOGF(LogScreenShotManager, Warning, "Fallback parent is malformed and has a leading '/'. This will be stripped but please correct. '%ls'", *Parent);
				Parent = Parent.RightChop(1);
			}

			// These are used as folders so ensure they match the expected layout
			if (Child.StartsWith(TEXT("/")))
			{
				UE_LOGF(LogScreenShotManager, Warning, "Fallback child is malformed and has a leading '/'. This will be stripped but please correct. '%ls'", *Child);
				Child = Child.RightChop(1);
			}

			FString& ParentValue = FallbackPlatforms.FindOrAdd(Child);
			ParentValue = Parent;
		}
	}

	// legacy path
	if (GConfig)
	{
		for (const FString& ConfigFilename : GConfig->GetFilenames())
		{
			const FConfigSection* FallbackSection = GConfig->GetSection(TEXT("AutomationTestFallbackHierarchy"), false, ConfigFilename);
			if (FallbackSection)
			{
				UE_LOGF(LogScreenShotManager, Warning, "Please move FallbackPlatform entries in [AutomationTestFallbackHierarchy] to +ScreenshotFallbackPlatforms= under[/Script/ScreenShotComparisonTools.ScreenShotComparisonSettings] in DefaultEngine.ini");

				// Parse all fallback definitions of the format "FallbackPlatform=(Child=/Platform/RHI, Parent=/Platform/RHI)"
				for (FConfigSection::TConstIterator Section(*FallbackSection); Section; ++Section)
				{
					if (Section.Key() == TEXT("FallbackPlatform"))
					{
						FString FallbackValue = Section.Value().GetValue();
						FString Child, Parent;
						bool bSuccess = false;

						if (FParse::Value(*FallbackValue, TEXT("Child="), Child, true) && FParse::Value(*FallbackValue, TEXT("Parent="), Parent, true))
						{
							if (Parent.StartsWith(TEXT("/")))
							{
								UE_LOGF(LogScreenShotManager, Warning, "Fallback parent is malformed and has a leading '/'. This will be stripped but please correct. '%ls'", *Parent);
								Parent = Parent.RightChop(1);
							}

							// These are used as folders so ensure they match the expected layout
							if (Child.StartsWith(TEXT("/")))
							{
								UE_LOGF(LogScreenShotManager, Warning, "Fallback child is malformed and has a leading '/'. This will be stripped but please correct. '%ls'", *Child);
								Child = Child.RightChop(1);
							}				
				
							// Append or override, could error here instead
							FString& Fallback = FallbackPlatforms.FindOrAdd(Child);
							Fallback = Parent;
							bSuccess = true;
						}
						
						if (!bSuccess)
						{
							UE_LOGF(LogScreenShotManager, Error, "Invalid fallback platform definition: '%ls'", *FallbackValue);
						}
					}
				}
			}
		}
	}
}

#undef LOCTEXT_NAMESPACE
