// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	ScreenShotManager.cpp: Implements the FScreenShotManager class.
=============================================================================*/

#pragma once

#include "CoreMinimal.h"
#include "Async/Future.h"
#include "ImageComparer.h"
#include "Interfaces/IScreenShotManager.h"
#include "MessageEndpoint.h"

DECLARE_LOG_CATEGORY_EXTERN(LogScreenShotManager, Log, All);

class FScreenshotComparisons;
struct FScreenShotDataItem;
class UScreenShotComparisonSettings;

/**
 * Implements the ScreenShotManager that contains screen shot data.
 */
class FScreenShotManager : public IScreenShotManager 
{
public:

	enum class EApprovedFolderOptions : int8
	{
		None = 0,
		UsePlatformFolders	= 1 << 0
	};

	/**
	 * Creates and initializes a new instance.
	 *
	 */
	FScreenShotManager();

	~FScreenShotManager();

public:

	//~ Begin IScreenShotManager Interface

	virtual TFuture<FImageComparisonResult> CompareScreenshotAsync(const FString& IncomingPath, const FAutomationScreenshotMetadata& MetaData, const EScreenShotCompareOptions Options) override;

	virtual FScreenshotExportResult ExportScreenshotComparisonResult(FString ScreenshotName, FString ExportPath = TEXT(""), bool bOnlyGeneratedFiles = false) override;

	TFuture<TSharedPtr<TArray<FComparisonReport>>> OpenComparisonReportsAsync(const FString& ImportPath) override;
	
	virtual FString GetIdealApprovedFolderForImage(const FAutomationScreenshotMetadata& MetaData) const override;

	virtual TArray<FString> FindApprovedFiles(const FAutomationScreenshotMetadata& IncomingMetaData, const FString& FilePattern) const override;

	virtual TSharedPtr<FImageComparisonResult> CompareImageSequence(const TMap<FString, FString>& Sequence, const FAutomationScreenshotMetadata& Metadata) override;

	virtual void NotifyAutomationTestFrameworkOfImageComparison(const FImageComparisonResult& Result) override;

	//~ End IScreenShotManager Interface

private:

	FString	GetPathComponentForRHI(const FAutomationScreenshotMetadata& MetaData) const;
	FString	GetPathComponentForPlatformAndRHI(const FAutomationScreenshotMetadata& MetaData) const;
	FString GetPathComponentForTestImages(const FAutomationScreenshotMetadata& MetaData, bool bIncludeVariantName) const;
		

	FString GetApprovedFolderForImageWithOptions(const FAutomationScreenshotMetadata& MetaData, EApprovedFolderOptions InOptions) const;

	FString GetDefaultExportDirectory() const;

	FImageComparisonResult CompareScreenshot(const FString& IncomingPath, const FAutomationScreenshotMetadata& MetaData, const EScreenShotCompareOptions Options);

	void CopyDirectory(const FString& DestDir, const FString& SrcDir, const TArray<FString> Patterns);

	void BuildFallbackPlatformsListFromConfig();

private:

	FString ScreenshotTempDeltaFolder;
	FString ScreenshotResultsFolder;
	TSharedPtr<TArray<FString>> PendingComparisonReportPaths;

	TMap<FString, FString> FallbackPlatforms;

	bool bUseConfidentialPlatformPaths;
};

inline FScreenShotManager::EApprovedFolderOptions operator | (FScreenShotManager::EApprovedFolderOptions lhs, FScreenShotManager::EApprovedFolderOptions rhs)
{
	return static_cast<FScreenShotManager::EApprovedFolderOptions>(static_cast<int8>(lhs) | static_cast<int8>(rhs));
}

inline bool operator & (FScreenShotManager::EApprovedFolderOptions& lhs, FScreenShotManager::EApprovedFolderOptions rhs)
{
	return (static_cast<int8>(lhs) & static_cast<int8>(rhs)) != 0;
}