// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	ScreenShotToolsModule.cpp: Implements the FScreenShotToolsModule class.
=============================================================================*/

#include "CoreMinimal.h"
#include "Modules/ModuleManager.h"
#include "Interfaces/IScreenShotToolsModule.h"
#include "ScreenShotManager.h"


/**
 * Implements the ScreenshotTools module.
 */
class FScreenShotToolsModule
	: public IScreenShotToolsModule
{
public:

	// Begin IScreenShotToolsModule interface

	virtual IScreenShotManagerPtr GetScreenShotManager() override
	{
		return ScreenShotManager;
	}

	// End IScreenShotToolsModule interface

public:

	/**
	 * Shutdown the screen shot manager tools module
	 */
	virtual void ShutdownModule( ) override
	{
		ScreenShotManager.Reset();
	}

	/**
	* Startup the screen shot manager tools module
	*/
	void StartupModule() override
	{
		// Create the screen shot manager
		if (!ScreenShotManager.IsValid())
		{
			ScreenShotManager = MakeShareable(new FScreenShotManager());
		}
	}

private:
	
	// Holds the screen shot manager.
	IScreenShotManagerPtr ScreenShotManager;
};


IMPLEMENT_MODULE(FScreenShotToolsModule, ScreenShotComparisonTools);
