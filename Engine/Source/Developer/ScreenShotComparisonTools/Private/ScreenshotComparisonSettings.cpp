// Copyright Epic Games, Inc. All Rights Reserved.


#include "ScreenShotComparisonSettings.h"

#if WITH_EDITOR
#include "Misc/CoreMisc.h"
#include "Misc/DataDrivenPlatformInfoRegistry.h"
#endif // WITH_EDITOR

UScreenShotComparisonSettings::UScreenShotComparisonSettings(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	//DefaultScreenshotResolution = FIntPoint(1920, 1080);
}

void UScreenShotComparisonSettings::LoadSettings()
{
	ScreenshotFallbackPlatforms.Empty();
	UObject::LoadConfig(GetClass());
}

UScreenShotComparisonSettings* UScreenShotComparisonSettings::Create(const FString& PlatformName)
{
	UScreenShotComparisonSettings* Settings = NewObject<UScreenShotComparisonSettings>();
	Settings->SetPlatform(PlatformName);
	return Settings;
}

const TCHAR* UScreenShotComparisonSettings::GetConfigOverridePlatform() const
{
	return *Platform;
}

const FString& UScreenShotComparisonSettings::GetPlatformName() const
{
	return Platform;
}

void UScreenShotComparisonSettings::SetPlatform(const FString& PlatformName)
{
	Platform = PlatformName;
	LoadSettings();
}

#if WITH_EDITOR
const TSet<FScreenshotFallbackEntry>& UScreenShotComparisonSettings::GetAllPlatformSettings()
{
	static TOptional<TSet<FScreenshotFallbackEntry>> ScreenshotFallbackEntries;

	if (!ScreenshotFallbackEntries.IsSet())
	{
		ScreenshotFallbackEntries.Emplace();

		for (const auto& Pair : FDataDrivenPlatformInfoRegistry::GetAllPlatformInfos())
		{
			if (Pair.Value.bIsFakePlatform || Pair.Value.bEnabledForUse == false)
			{
				continue;
			}

			FString PlatformName = Pair.Key.ToString();
			UScreenShotComparisonSettings* Settings (UScreenShotComparisonSettings::Create(PlatformName));
			check(nullptr != Settings);

			ScreenshotFallbackEntries->Append(Settings->ScreenshotFallbackPlatforms);
		}
	}

	return ScreenshotFallbackEntries.GetValue();
}

#endif // WITH_EDITOR
