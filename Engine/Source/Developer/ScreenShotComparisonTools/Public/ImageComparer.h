// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/ContainersFwd.h"
#include "Containers/UnrealString.h"
#include "CoreMinimal.h"
#include "Internationalization/Text.h"
#include "Math/Color.h"
#include "Math/UnrealMathSSE.h"
#include "Misc/AssertionMacros.h"
#include "Misc/DateTime.h"
#include "Misc/Paths.h"
#include "Templates/SharedPointer.h"
#include "UObject/ObjectMacros.h"

#include "ImageComparer.generated.h"

#define UE_API SCREENSHOTCOMPARISONTOOLS_API

class Error;
class FComparableImage;

/**
 * 
 */
USTRUCT()
struct FImageTolerance
{
	GENERATED_USTRUCT_BODY()

public:

	UPROPERTY(EditAnywhere, Category = Automation)
	uint8 Red;

	UPROPERTY(EditAnywhere, Category = Automation)
	uint8 Green;

	UPROPERTY(EditAnywhere, Category = Automation)
	uint8 Blue;

	UPROPERTY(EditAnywhere, Category = Automation)
	uint8 Alpha;

	UPROPERTY(EditAnywhere, Category = Automation)
	uint8 MinBrightness;

	UPROPERTY(EditAnywhere, Category = Automation)
	uint8 MaxBrightness;

	UPROPERTY(EditAnywhere, Category = Automation)
	bool IgnoreAntiAliasing;

	UPROPERTY(EditAnywhere, Category = Automation)
	bool IgnoreColors;

	UPROPERTY(EditAnywhere, Category = Automation)
	float MaximumLocalError;

	UPROPERTY(EditAnywhere, Category = Automation)
	float MaximumGlobalError;

	FImageTolerance()
		: Red(0)
		, Green(0)
		, Blue(0)
		, Alpha(0)
		, MinBrightness(0)
		, MaxBrightness(255)
		, IgnoreAntiAliasing(false)
		, IgnoreColors(false)
		, MaximumLocalError(0.0f)
		, MaximumGlobalError(0.0f)
	{
	}

	FImageTolerance(uint8 R, uint8 G, uint8 B, uint8 A, uint8 InMinBrightness, uint8 InMaxBrightness, bool InIgnoreAntiAliasing, bool InIgnoreColors, float InMaximumLocalError, float InMaximumGlobalError)
		: Red(R)
		, Green(G)
		, Blue(B)
		, Alpha(A)
		, MinBrightness(InMinBrightness)
		, MaxBrightness(InMaxBrightness)
		, IgnoreAntiAliasing(InIgnoreAntiAliasing)
		, IgnoreColors(InIgnoreColors)
		, MaximumLocalError(InMaximumLocalError)
		, MaximumGlobalError(InMaximumGlobalError)
	{
	}

public:
	UE_API const static FImageTolerance DefaultIgnoreNothing;
	UE_API const static FImageTolerance DefaultIgnoreLess;
	UE_API const static FImageTolerance DefaultIgnoreAntiAliasing;
	UE_API const static FImageTolerance DefaultIgnoreColors;
};

UENUM()
enum class EImageTolerancePreset : uint8
{
	IgnoreNothing,
	IgnoreLess,
	IgnoreAntiAliasing,
	IgnoreColors,
	Custom
};

static FImageTolerance GetImageToleranceForPreset(EImageTolerancePreset Preset, FImageTolerance CustomTolerance)
{
	switch (Preset)
	{
	case EImageTolerancePreset::IgnoreNothing: return FImageTolerance::DefaultIgnoreNothing;
	case EImageTolerancePreset::IgnoreLess: return FImageTolerance::DefaultIgnoreLess;
	case EImageTolerancePreset::IgnoreAntiAliasing: return FImageTolerance::DefaultIgnoreAntiAliasing;
	case EImageTolerancePreset::IgnoreColors: return FImageTolerance::DefaultIgnoreColors;
	case EImageTolerancePreset::Custom: return CustomTolerance;
	default:
		check(0);
	}

	return FImageTolerance();
}

class FComparableImage;

class FPixelOperations
{
public:
	static FORCEINLINE double GetLuminance(const FColor& Color)
	{
		// https://en.wikipedia.org/wiki/Relative_luminance
		return (0.2126 * Color.R + 0.7152 * Color.G + 0.0722 * Color.B) * (Color.A / 255.0);
	}

	static bool IsBrightnessSimilar(const FColor& ColorA, const FColor& ColorB, const FImageTolerance& Tolerance)
	{
		const bool AlphaSimilar = FMath::IsNearlyEqual((float)ColorA.A, ColorB.A, Tolerance.Alpha);

		const double BrightnessA = FPixelOperations::GetLuminance(ColorA);
		const double BrightnessB = FPixelOperations::GetLuminance(ColorB);
		const bool BrightnessSimilar = FMath::IsNearlyEqual(BrightnessA, BrightnessB, Tolerance.MinBrightness);

		return BrightnessSimilar && AlphaSimilar;
	}

	static FORCEINLINE bool IsRGBSame(const FColor& ColorA, const FColor& ColorB)
	{
		return ColorA.R == ColorB.R &&
			ColorA.G == ColorB.G &&
			ColorA.B == ColorB.B;
	}

	static FORCEINLINE bool IsRGBSimilar(const FColor& ColorA, const FColor& ColorB, const FImageTolerance& Tolerance)
	{
		const bool RedSimilar = FMath::IsNearlyEqual((float)ColorA.R, ColorB.R, Tolerance.Red);
		const bool GreenSimilar = FMath::IsNearlyEqual((float)ColorA.G, ColorB.G, Tolerance.Green);
		const bool BlueSimilar = FMath::IsNearlyEqual((float)ColorA.B, ColorB.B, Tolerance.Blue);
		const bool AlphaSimilar = FMath::IsNearlyEqual((float)ColorA.A, ColorB.A, Tolerance.Alpha);

		return RedSimilar && GreenSimilar && BlueSimilar && AlphaSimilar;
	}

	static FORCEINLINE bool IsContrasting(const FColor& ColorA, const FColor& ColorB, const FImageTolerance& Tolerance)
	{
		const double BrightnessA = FPixelOperations::GetLuminance(ColorA);
		const double BrightnessB = FPixelOperations::GetLuminance(ColorB);

		return FMath::Abs(BrightnessA - BrightnessB) > Tolerance.MaxBrightness;
	}

	static float GetHue(const FColor& Color);

	static bool IsAntialiased(const FColor& SourcePixel, const FComparableImage* Image, int32 X, int32 Y, const FImageTolerance& Tolerance);
};

/**
 *
 */
class FComparableImage
{
public:
	int32 Width = 0;
	int32 Height = 0;
	TArray64<uint8> Bytes;

	FComparableImage()
	{
	}

	FORCEINLINE bool CanGetPixel(int32 X, int32 Y) const
	{
		return X >= 0 && Y >= 0 && X < Width && Y < Height;
	}

	FORCEINLINE FColor GetPixel(int32 X, int32 Y) const
	{
		int64 Offset = ( (int64)Y * Width + X ) * 4;
		check(Offset < ( (int64)Width * Height * 4 ));

		return FColor(
			Bytes[Offset],
			Bytes[Offset + 1],
			Bytes[Offset + 2],
			Bytes[Offset + 3]);
	}

	/**
	 * Populate image by loading an file
	 *
	 * @param ImagePath Path for the image file to load
	 * @param OutError Contains the error message if load fails
	 * 
	 * @return true if success
	*/
	UE_API bool LoadFile(const FString& ImagePath, FText& OutError);

	/**
	 * Populate image by loading compressed data
	 *
	 * @param CompressedData The memory address of the start of the compressed data.
	 * @param CompressedSize The size of the compressed data parsed.
	 * @param ImageExtension File extension of the image format
	 * @param OutError Contains the error message if load fails
	 * 
	 * @return true if success
	*/
	UE_API bool LoadCompressedData(const void* CompressedData, int64 CompressedSize, const FString& ImageExtension, FText& OutError);
};

/**
 * This struct holds the results of comparing an incoming image from a platform with an approved image that exists under the 
 * project hierarchy. 
 
 * All paths in this structure should be portable. Test results (including this struct) result can be serialized to 
 * JSON and stored on the network as during automation runs then opened in the editor to commit / approve changes
 * to the local project.
 */
USTRUCT()
struct FImageComparisonResult
{
	GENERATED_USTRUCT_BODY()

public:

	/*
		Time that the comparison was performed
	*/
	UPROPERTY()
	FDateTime CreationTime;

	/*
		Platform that the incoming image was generated on
	*/
	UPROPERTY()
	FString SourcePlatform;

	/*
		RHI that the incoming image was generated with
	*/
	UPROPERTY()
	FString SourceRHI;

	/*
		Path to a folder where the idealized ground-truth for this comparison would be. Relative to the project directory.
		Note: This path may not exist a fallback is being used for approval, or if there is no approved
		image at all. Comparing this value with the FPaths::GetPath(ApprovedFilePath) can be used to determine that.
		(the IsIdeal() function performs that check).
	*/
	UPROPERTY()
	FString IdealApprovedFolderPath;

	/*
		Path to the file that was considered as the ground-truth. Relative to the project directory
	*/
	UPROPERTY()
	FString ApprovedFilePath;

	/*
		Path to the file that was generated in the test. Relative to the project directory, only valid when a test is run locally
	*/
	UPROPERTY()
	FString IncomingFilePath;

	/*
		Path to the delta image between the ground-truth and the incoming file. Relative to the project directory, only valid when a test is run locally
	*/
	UPROPERTY()
	FString ComparisonFilePath;

	/*
		Name of the approved file saved for the report. Path is relative to the location of the metadata for the report
	*/
	UPROPERTY()
	FString ReportApprovedFilePath;

	/*
		name of the incoming file saved for the report.  Path is relative to the location of the metadata for the report
	*/
	UPROPERTY()
	FString ReportIncomingFilePath;

	/*
		Name of the delta image saved for the report.  Path is relative to the location of the metadata for the report
	*/
	UPROPERTY()
	FString ReportComparisonFilePath;

	/*
		Largest local difference found during comparison
	*/
	UPROPERTY()
	double MaxLocalDifference;

	/*
		Global difference found during comparison
	*/
	UPROPERTY()
	double GlobalDifference;

	/*
		Tolerance values for comparison
	*/
	UPROPERTY()
	FImageTolerance Tolerance;

	/*
		Error message that can be set during a comparison
	*/
	UPROPERTY()
	FText ErrorMessage;

	/*
		Path of the screenshot (includes variant if applicable)
	*/
	UPROPERTY()
	FString ScreenshotPath;

	/*
		Id of the comparison request
	*/
	UPROPERTY()
	FGuid ComparisonId;

	/*
		Whether to skip saving and attaching images to the report for this test
	*/
	UPROPERTY()
	bool bSkipAttachingImages;

	/*
		Version of the image comparision result 
	*/
	UPROPERTY()
	int32 Version;

	static constexpr int32 CurrentVersion = 3;
	static constexpr int32 OldestSupportedVersion = 2;

	FImageComparisonResult()
		: CreationTime(0)
		, MaxLocalDifference(0.0f)
		, GlobalDifference(0.0f)
		, ErrorMessage()
		, bSkipAttachingImages(false)
		, Version(CurrentVersion)
	{
	}

	FImageComparisonResult(const FText& Error)
		: CreationTime(0)
		, MaxLocalDifference(0.0f)
		, GlobalDifference(0.0f)
		, ErrorMessage(Error)
		, bSkipAttachingImages(false)
		, Version(CurrentVersion)
	{
	}

	/*
		Returns true if this is a new image with no approved file to compare against
	*/
	bool IsValid() const
	{
		return Version >= OldestSupportedVersion && Version <= CurrentVersion;
	}
	
	/*
		Marks this struct as invalid. Can be used before serializing in to ensure very old files with
		no version info aren't recognized.
	*/
	void SetInvalid()
	{
		Version = 0;
	}

	/*
		Returns true if this is a new image with no approved file to compare against
	*/
	bool IsNew() const
	{
		return ApprovedFilePath.IsEmpty();
	}

	/*
		Returns true if this is am ideal comparison (e.g not using a fallback for
		comparison)
	*/
	bool IsIdeal() const
	{
		return FPaths::GetPath(ApprovedFilePath) == IdealApprovedFolderPath;
	}

	/*
		Returns true if the images were within the provided tolerance values
	*/
	bool AreSimilar() const
	{
		if ( IsNew() )
		{
			return false;
		}

		if (!ErrorMessage.IsEmpty())
		{
			return false;
		}

		if ( MaxLocalDifference > Tolerance.MaximumLocalError || GlobalDifference > Tolerance.MaximumGlobalError )
		{
			return false;
		}

		return true;
	}
};

struct FComparisonReport
{
public:

	/*
		ReportRootDirectory is where all reports are saved. E.g. <path>/Saved/Automation/Reports
		ReportFile is a specific report for a test under this path. E.g. <path>/Saved/Automation/Reports/Test/TestName/report.json.
	*/

	UE_API FComparisonReport(const FString& InReportRootDirectory, const FString& InReportFile);

	void SetComparisonResult(const FImageComparisonResult& InResult)
	{
		Comparison = InResult;
	}

	const FImageComparisonResult& GetComparisonResult() const
	{
		return Comparison;
	}

	/*
		Return the path to the file used to generate this report
	*/
	const FString& GetReportFile() const { return ReportFile; }

	/*
		Return the path to all files in this report
	*/
	const FString& GetReportPath() const { return ReportPath; }

	/*
		Return the path to a location where all reports (including this one) are kept in this session
	*/
	const FString& GetReportRootDirectory() const { return ReportRootDirectory; }

private:

	FString ReportRootDirectory;
	FString ReportFile;
	FString ReportPath;
	FImageComparisonResult Comparison;
};

/**
 * 
 */
class FImageComparer
{
public:
	
	UE_API FImageComparisonResult Compare(const FString& ImagePathA, const FString& ImagePathB, FImageTolerance Tolerance, const FString& OutDeltaPath);
	UE_API FImageComparisonResult Compare(const FComparableImage* ImageA, const FComparableImage* ImageB, FImageTolerance Tolerance, const FString& OutDeltaPath);

	enum class EStructuralSimilarityComponent : uint8
	{
		Luminance,
		Color
	};

	/**
	 * https://en.wikipedia.org/wiki/Structural_similarity
	 */
	UE_API double CompareStructuralSimilarity(const FString& ImagePathA, const FString& ImagePathB, EStructuralSimilarityComponent InCompareComponent, const FString& OutDeltaPath);
	UE_API double CompareStructuralSimilarity(const FComparableImage* ImageA, const FComparableImage* ImageB, EStructuralSimilarityComponent InCompareComponent, const FString& OutDeltaPath);
};

#undef UE_API
