// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	IScreenShotToolsModule.h: Declares the IScreenShotToolsModule interface.
=============================================================================*/

#pragma once

#include "CoreMinimal.h"
#include "Modules/ModuleInterface.h"
#include "Interfaces/IScreenShotManager.h"

/**
 * Interface for session core modules.
 */
class IScreenShotToolsModule
	: public IModuleInterface
{
public:

	/**
	* Get the session manager.
	*
	* @return The session manager.
	*/
	virtual IScreenShotManagerPtr GetScreenShotManager( ) = 0;

public:

	/**
	 * Virtual destructor.
	 */
	virtual ~IScreenShotToolsModule( ) = default;
};
