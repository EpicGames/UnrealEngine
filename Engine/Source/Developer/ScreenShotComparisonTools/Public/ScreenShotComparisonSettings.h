// Copyright Epic Games, Inc. All Rights Reserved.


#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "UObject/Object.h"
#include "ScreenShotComparisonSettings.generated.h"

#define UE_API SCREENSHOTCOMPARISONTOOLS_API

/**
* Holds settings for screenshot fallbacks
*/
USTRUCT()
struct FScreenshotFallbackEntry
{
	GENERATED_USTRUCT_BODY()

	UPROPERTY(Config)
	FString Parent;

	UPROPERTY(Config)
	FString Child;

	bool operator==(const FScreenshotFallbackEntry& Other) const
	{
		return Child == Other.Child;
	}
};

FORCEINLINE uint32 GetTypeHash(const FScreenshotFallbackEntry& Object)
{
	return GetTypeHash(Object.Child);
}

UCLASS(MinimalAPI, config = Engine, defaultconfig)
class UScreenShotComparisonSettings : public UObject
{
	GENERATED_UCLASS_BODY()

public:

	/**
	* If true, any checked-in test results for confidential platforms will be put under <ProjectDir>/Platforms/<Platform>/Test instead of <ProjectDir>/Test
	*/
	UPROPERTY(Config)
	bool bUseConfidentialPlatformPathsForSavedResults;

	/**
	* An array of entries that describe other platforms we can use for fallbacks when comparing screenshots
	*/
	UPROPERTY(Config)
	TArray<FScreenshotFallbackEntry>  ScreenshotFallbackPlatforms;

	/**
	* Creates class instance
	* @param PlatformName Reference to a string containing platform name (if it is empty the current platform name is used).
	*/
	static UE_API UScreenShotComparisonSettings* Create(const FString& PlatformName = FString{});

	/**
	 * Loads settings of corresponding config.
	 */
	UE_API virtual void LoadSettings() final;

	/**
	 * Overrides config hierarchy platform to be used in UObject internals
	 */
	UE_API virtual const TCHAR* GetConfigOverridePlatform() const override;

#if WITH_EDITOR
public:
	static UE_API const TSet<FScreenshotFallbackEntry>& GetAllPlatformSettings();
#endif // WITH_EDITOR

protected:

	/**
	 * Returns platform name reference. As the class can store platform-independent config, it returns an empty string if the platform was not specified.
	 */
	UE_API virtual const FString& GetPlatformName() const;

	/**
	 * Sets platform and reloads settings.
	 * @param PlatformName Reference to a string containing platform name (if it is empty the default config is used).
	 */
	UE_API virtual void SetPlatform(const FString& PlatformName);

private:
	FString Platform;
};

#undef UE_API
