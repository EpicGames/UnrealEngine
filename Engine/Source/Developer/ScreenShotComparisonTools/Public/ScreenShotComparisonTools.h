// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	ScreenShotComparisonTools.h: ScreenShotComparisonTools module public header file.
=============================================================================*/

#pragma once


/* Boilerplate
 *****************************************************************************/

#include "Misc/MonolithicHeaderBoilerplate.h"
MONOLITHIC_HEADER_BOILERPLATE()

/* Dependencies
 *****************************************************************************/

#include "Core.h"
#include "Modules/ModuleManager.h"
#include "Misc/IFilter.h"
#include "Misc/TextFilter.h"
#include "Misc/FilterCollection.h"

/* Interfaces
 *****************************************************************************/
#include "ImageComparer.h"
#include "Interfaces/IScreenShotManager.h"
#include "Interfaces/IScreenShotToolsModule.h"
