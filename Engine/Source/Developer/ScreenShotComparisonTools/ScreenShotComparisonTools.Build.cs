// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class ScreenShotComparisonTools : ModuleRules
{
	public ScreenShotComparisonTools(ReadOnlyTargetRules Target) : base(Target)
	{
		PublicDependencyModuleNames.AddRange(
			new string[] {
				"Core",
				"CoreUObject"
			}
		);

		PrivateDependencyModuleNames.AddRange(
			new string[] {
				"AutomationMessages",
				
				"ImageWrapper",
				"Json",
				"JsonUtilities",
				"DesktopPlatform",
				"RenderCore"
			}
		);

		PrivateIncludePathModuleNames.AddRange(
			new string[] {
				"MessagingCommon",
				"LauncherServices"
			}
		);

		if (Target.bCompileAgainstEngine && Target.Configuration != UnrealTargetConfiguration.Shipping)
		{
			PrecompileForTargets = PrecompileTargetsType.Any;
		}
	}
}
